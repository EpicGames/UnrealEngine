{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Live Link Hub Example Device",
	"Description": "An example device implementation using the Live Link Hub device interface",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Plugins": [
		{
			"Name": "LiveLinkDevice",
			"Enabled": true
		}
	],
	"Modules": [
		{
			"Name": "LiveLinkExampleDevice",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	]
}
