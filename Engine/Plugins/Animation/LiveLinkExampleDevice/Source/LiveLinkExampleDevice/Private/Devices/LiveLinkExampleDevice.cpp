// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkExampleDevice.h"
#include "Engine/Engine.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "LiveLinkDeviceSubsystem.h"
#include "LiveLinkHub/ILiveLinkRecordingSession.h"
#include "Logging/StructuredLog.h"
#include "Modules/ModuleManager.h"


IMPLEMENT_MODULE(FDefaultModuleImpl, LiveLinkExampleDevice);


DECLARE_LOG_CATEGORY_EXTERN(LogLiveLinkExampleDevice, Verbose, All);

DEFINE_LOG_CATEGORY(LogLiveLinkExampleDevice);


#define LOCTEXT_NAMESPACE "LiveLinkExampleDevice"


void ULiveLinkExampleDevice::OnDeviceAdded()
{
}


void ULiveLinkExampleDevice::OnDeviceRemoved()
{
}


EDeviceHealth ULiveLinkExampleDevice::GetDeviceHealth() const
{
	return EDeviceHealth::Nominal;
}

FText ULiveLinkExampleDevice::GetHealthText() const
{
	return FText::FromString("Example Health");
}


void ULiveLinkExampleDevice::OnSettingChanged(const FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::OnSettingChanged(InPropertyChangedEvent);

	const ULiveLinkExampleDeviceSettings* DeviceSettings = GetDeviceSettings<ULiveLinkExampleDeviceSettings>();

	static const FName AddressName = GET_MEMBER_NAME_CHECKED(ULiveLinkExampleDeviceSettings, IpAddress);
	static const FName PortName = GET_MEMBER_NAME_CHECKED(ULiveLinkExampleDeviceSettings, Port);

	const FName ChangedPropertyName = InPropertyChangedEvent.GetPropertyName();
	if (ChangedPropertyName == AddressName)
	{
		FIPv4Address Address;
		if (FIPv4Address::Parse(DeviceSettings->IpAddress, Address))
		{
			UE_LOGFMT(LogLiveLinkExampleDevice, Verbose, "Device '{Name}': Changing address to {Address}",
				DeviceSettings->DisplayName, Address.ToString());
		}
		else
		{
			UE_LOGFMT(LogLiveLinkExampleDevice, Warning, "Device '{Name}': Failed to parse address {Address}",
				DeviceSettings->DisplayName, DeviceSettings->IpAddress);
		}
	}
	else if (ChangedPropertyName == PortName)
	{
		UE_LOGFMT(LogLiveLinkExampleDevice, Verbose, "Device '{Name}': Changing port to {Port}",
			DeviceSettings->DisplayName, DeviceSettings->Port);

	}
}


ELiveLinkDeviceConnectionStatus ULiveLinkExampleDevice::GetConnectionStatus_Implementation() const
{
	return ELiveLinkDeviceConnectionStatus::Disconnected;
}


FString ULiveLinkExampleDevice::GetHardwareId_Implementation() const
{
	return GetDeviceSettings<ULiveLinkExampleDeviceSettings>()->IpAddress;
}


bool ULiveLinkExampleDevice::SetHardwareId_Implementation(const FString& InHardwareID)
{
	ULiveLinkExampleDeviceSettings* DeviceSettings = GetDeviceSettings<ULiveLinkExampleDeviceSettings>();

	bool bParsedEndpoint = false;
	FIPv4Endpoint Endpoint;
	if (FIPv4Endpoint::Parse(InHardwareID, Endpoint))
	{
		bParsedEndpoint = true;
	}
	else if (FIPv4Address::Parse(InHardwareID, Endpoint.Address))
	{
		const UClass* SettingsClass = ULiveLinkExampleDeviceSettings::StaticClass();
		const ULiveLinkExampleDeviceSettings* SettingsCDO =
			SettingsClass->GetDefaultObject<ULiveLinkExampleDeviceSettings>();
		const uint16 DefaultPort = SettingsCDO->Port;

		Endpoint.Port = DefaultPort;

		bParsedEndpoint = true;
	}

	if (bParsedEndpoint)
	{
		UE_LOGFMT(LogLiveLinkExampleDevice, Verbose, "Device '{Name}': Changing endpoint to {Endpoint}",
			DeviceSettings->DisplayName, Endpoint.ToString());

		DeviceSettings->IpAddress = Endpoint.Address.ToString();
		DeviceSettings->Port = Endpoint.Port;

		return true;
	}
	else
	{
		UE_LOGFMT(LogLiveLinkExampleDevice, Warning, "Device '{Name}': Failed to parse endpoint {HardwareID}",
			DeviceSettings->DisplayName, InHardwareID);

		return false;
	}
}


bool ULiveLinkExampleDevice::Connect_Implementation()
{
	if (!ensure(ConnectionStatus == ELiveLinkDeviceConnectionStatus::Disconnected))
	{
		return false;
	}

	const ULiveLinkExampleDeviceSettings* DeviceSettings = GetDeviceSettings<ULiveLinkExampleDeviceSettings>();

	FIPv4Endpoint Endpoint;
	if (!FIPv4Address::Parse(DeviceSettings->IpAddress, Endpoint.Address))
	{
		return false;
	}

	Endpoint.Port = DeviceSettings->Port;

	ConnectionStatus = ELiveLinkDeviceConnectionStatus::Connecting;
	SetConnectionStatus(ConnectionStatus);

	return true;
}


bool ULiveLinkExampleDevice::Disconnect_Implementation()
{
	const bool bCanDisconnect = (
		ConnectionStatus == ELiveLinkDeviceConnectionStatus::Connected ||
		ConnectionStatus == ELiveLinkDeviceConnectionStatus::Connecting);

	if (!ensure(bCanDisconnect))
	{
		return false;
	}

	ConnectionStatus = ELiveLinkDeviceConnectionStatus::Disconnected;

	return false;
}


bool ULiveLinkExampleDevice::StartRecording_Implementation()
{
	const ULiveLinkExampleDeviceSettings* DeviceSettings = GetDeviceSettings<ULiveLinkExampleDeviceSettings>();
	ILiveLinkRecordingSession& SessionInfo = ILiveLinkRecordingSession::Get();

	UE_LOGFMT(LogLiveLinkExampleDevice, Verbose,
		"Device '{Name}': Started recording ({Session} / {Slate} / {Take})",
		DeviceSettings->DisplayName,
		SessionInfo.GetSlateName(),
		SessionInfo.GetTakeNumber(),
		SessionInfo.GetSessionName()
	);

	bIsRecording = true;

	return true;
}


bool ULiveLinkExampleDevice::StopRecording_Implementation()
{
	bIsRecording = false;

	return true;
}


bool ULiveLinkExampleDevice::IsRecording_Implementation() const
{
	return bIsRecording;
}


#undef LOCTEXT_NAMESPACE
