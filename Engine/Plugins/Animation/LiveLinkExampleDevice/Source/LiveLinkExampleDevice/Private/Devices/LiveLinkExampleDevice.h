// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkDevice.h"
#include "LiveLinkDeviceCapability_Connection.h"
#include "LiveLinkDeviceCapability_Recording.h"
#include "LiveLinkExampleDevice.generated.h"


UCLASS()
class ULiveLinkExampleDeviceSettings : public ULiveLinkDeviceSettings
{
    GENERATED_BODY()

public:
	
	ULiveLinkExampleDeviceSettings()
	{
		DisplayName = TEXT("Example Device");
	}

	UPROPERTY(EditAnywhere, Category="Example Device")
	FString IpAddress = TEXT("127.0.0.1");

	UPROPERTY(EditAnywhere, Category="Example Device")
	uint16 Port = 12345;
};


UCLASS()
class ULiveLinkExampleDevice : public ULiveLinkDevice
	, public ILiveLinkDeviceCapability_Connection
	, public ILiveLinkDeviceCapability_Recording
{
	GENERATED_BODY()

public:
	//~ Begin ULiveLinkDevice interface
	virtual TSubclassOf<ULiveLinkDeviceSettings> GetSettingsClass() const override
	{
		return ULiveLinkExampleDeviceSettings::StaticClass();
	}

	virtual EDeviceHealth GetDeviceHealth() const override;
	virtual FText GetHealthText() const override;

	virtual void OnDeviceAdded() override;
	virtual void OnDeviceRemoved() override;
	virtual void OnSettingChanged(const FPropertyChangedEvent& InPropertyChangedEvent) override;
	//~ End ULiveLinkDevice interface

	//~ Begin ILiveLinkDeviceCapability_Connection interface
	virtual ELiveLinkDeviceConnectionStatus GetConnectionStatus_Implementation() const override;
	virtual FString GetHardwareId_Implementation() const override;
	virtual bool SetHardwareId_Implementation(const FString& HardwareID) override;
	virtual bool Connect_Implementation() override;
	virtual bool Disconnect_Implementation() override;
	//~ End ILiveLinkDeviceCapability_Connection interface

	//~ Begin ILiveLinkDeviceCapability_Recording interface
	virtual bool StartRecording_Implementation() override;
	virtual bool StopRecording_Implementation() override;
	virtual bool IsRecording_Implementation() const override;
	//~ End ILiveLinkDeviceCapability_Recording interface

private:
	ELiveLinkDeviceConnectionStatus ConnectionStatus = ELiveLinkDeviceConnectionStatus::Disconnected;
	bool bIsRecording = false;
};
