// Copyright Epic Games, Inc. All Rights Reserved.

#include "Devices/LiveLinkExampleDevice.h"
#include "Engine/Engine.h"
#include "LiveLinkDeviceSubsystem.h"
#include "Misc/AutomationTest.h"


IMPLEMENT_SIMPLE_AUTOMATION_TEST(FExampleDeviceTest, "LiveLinkHub.ExampleDevice",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FExampleDeviceTest::RunTest(const FString& Parameters)
{
	ULiveLinkDeviceSubsystem* DeviceSubsystem = GEngine->GetEngineSubsystem<ULiveLinkDeviceSubsystem>();

	ULiveLinkExampleDeviceSettings* DeviceSettingsTemplate = NewObject<ULiveLinkExampleDeviceSettings>();
	DeviceSettingsTemplate->IpAddress = FString(TEXT("127.1.2.3"));
	DeviceSettingsTemplate->DisplayName = FString(TEXT("Test Device"));

	ULiveLinkDeviceSubsystem::FCreateResult CreateResult =
		DeviceSubsystem->CreateDeviceOfClass(ULiveLinkExampleDevice::StaticClass(), DeviceSettingsTemplate);

	if (!TestTrueExpr(CreateResult.HasValue()))
	{
		return false;
	}

	const FGuid DeviceId = CreateResult.GetValue().DeviceId;
	ULiveLinkDevice* NewDevice = CreateResult.GetValue().Device;

	return true;
}
